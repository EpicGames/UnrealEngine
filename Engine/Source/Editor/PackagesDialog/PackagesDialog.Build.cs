// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class PackagesDialog : ModuleRules
{
	public PackagesDialog(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("AssetTools");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core", 
				"CoreUObject", 
                "InputCore",
				"Slate", 
				"SlateCore",
				"UnrealEd",
				"SourceControl",
				"AssetRegistry",
				"ToolWidgets",
			}
		);

		DynamicallyLoadedModuleNames.Add("AssetTools");
	}
}
