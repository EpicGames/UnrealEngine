// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackagesDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWindow.h"
#include "Editor.h"
#include "SPackagesDialog.h"
#include "Widgets/Views/SListView.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Framework/Docking/TabManager.h"

IMPLEMENT_MODULE( FPackagesDialogModule, PackagesDialog );

const FVector2D FPackagesDialogModule::DEFAULT_WINDOW_SIZE = FVector2D(640, 492);	
const FVector2D FPackagesDialogModule::EXTRA_WINDOW_WIDTH = FVector2D(150, 0);

/**
 * Called right after the module's DLL has been loaded and the module object has been created
 */
void FPackagesDialogModule::StartupModule()
{
	FModuleManager::Get().OnModulesChanged().AddRaw( this, &FPackagesDialogModule::OnModulesChanged );
}

/**
 * Called before the module is unloaded, right before the module object is destroyed.
 */
void FPackagesDialogModule::ShutdownModule()
{
	FModuleManager::Get().OnModulesChanged().RemoveAll( this );

	/** Destroy the main frame window */
	TSharedPtr< SWindow > PinnedEditorPackagesDialogWindow( EditorPackagesDialogWindow.Pin() );
	if( PinnedEditorPackagesDialogWindow.IsValid() )
	{
		PinnedEditorPackagesDialogWindow->DestroyWindowImmediately();
	}
}

void FPackagesDialogModule::CreatePackagesDialog(const FText& Title, const FText& Message, bool InReadOnly, bool InAllowSourceControlConnection, const FSimpleDelegate& InOnSourceControlStateChanged)
{
	PackageDialogTitle = Title;
	PackagesDialogWidget = SNew(SPackagesDialog) 
		.ReadOnly(InReadOnly)
		.AllowSourceControlConnection(InAllowSourceControlConnection)
		.Message(Message)
		.OnSourceControlStateChanged(InOnSourceControlStateChanged);
}

/**
 * Shows the package dialog window as a modal window
 *
  * @return	Which button was pressed
 */
EDialogReturnType FPackagesDialogModule::ShowPackagesDialog()
{
	TSet<FString> DummyList;
	return ShowPackagesDialog(DummyList);
}

/**
 * Shows the package dialog window as a modal window
 *
  * @param	InOutPackagedToIgnore	The array that should be updated with the ignored packages
  * @return	Which button was pressed
 */
EDialogReturnType FPackagesDialogModule::ShowPackagesDialog(OUT TSet<FString>& InOutIgnoredPackages)
{
	/** Ensure the package dialog window was not created already */
	check(!EditorPackagesDialogWindow.IsValid());
	check(PackagesDialogWidget.IsValid());

	/** Reset the widget, as it may have been re-used */
	PackagesDialogWidget->Reset();

	/** Apply the current status of the ignore for save list to the items */
	PackagesDialogWidget->PopulateIgnoreForSaveItems(InOutIgnoredPackages);

	/** Create the window to host our package dialog widget */
	FVector2D WindowSize = DEFAULT_WINDOW_SIZE;
	if (PackagesDialogWidget->IsSourceControlConnectionAllowed())
	{
		// Make the window a little wider if there's an additional column for source control status
		WindowSize += EXTRA_WINDOW_WIDTH;
	}

	TSharedRef< SWindow > EditorPackagesDialogWindowRef = SNew(SWindow)
		.Title(PackageDialogTitle)
		.ClientSize(WindowSize);
	
	/** Store weak pointers to the package dialog Slate window and widget */
	EditorPackagesDialogWindow = EditorPackagesDialogWindowRef;

	/** Set the content of the window to our package dialog widget */
	EditorPackagesDialogWindowRef->SetContent(PackagesDialogWidget.ToSharedRef());

	/** Set the keyboard focus to the first button in package dialog so that pressing return will select the default option */
	TSharedPtr< SWidget > WidgetToFocusOn( PackagesDialogWidget->GetWidgetToFocusOnActivate() );
	if( WidgetToFocusOn.IsValid() )
	{
		EditorPackagesDialogWindowRef->SetWidgetToFocusOnActivate( WidgetToFocusOn );
	}

	/** If the editor is not in the foreground, draw attention to it so users know a dialog requires their interaction */
	if (TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow())
	{
		if (RootWindow->GetNativeWindow() && !RootWindow->GetNativeWindow()->IsForegroundWindow())
		{
			RootWindow->DrawAttention(FWindowDrawAttentionParameters());
		}
	}

	/** Show the package dialog window as a modal window */
	GEditor->EditorAddModalWindow( EditorPackagesDialogWindowRef );

	/** Clear the result package arrays */
	CheckedPackages.Empty();
	UncheckedPackages.Empty();
	UndeterminedPackages.Empty();

	/** Get the return type for the dialog and populate the result package arrays */
	const EDialogReturnType DialogReturnType = PackagesDialogWidget.Get()->GetReturnType(CheckedPackages, UncheckedPackages, UndeterminedPackages);

	/** After finishing with the dialog, update the ignored items list incase the user changed it */
	PackagesDialogWidget->PopulateIgnoreForSaveArray(InOutIgnoredPackages);

	return DialogReturnType;
}

/**
 * Removes the package dialog window
 */
void FPackagesDialogModule::RemovePackagesDialog()
{
	const TSharedPtr<SWindow> PinnedWindow = EditorPackagesDialogWindow.Pin();
	/** Ensure the package dialog has been created already */
	check( PinnedWindow.IsValid() );

	/** Request to destroy the package dialog window */
	if (PinnedWindow)
	{
		PinnedWindow->RequestDestroyWindow();
	}
}

void FPackagesDialogModule::SetMessage(const FText& InMessage)
{
	check(PackagesDialogWidget.IsValid());
	PackagesDialogWidget->SetMessage(InMessage);
}

void FPackagesDialogModule::SetWarning(const FText& InMessage)
{
	check(PackagesDialogWidget.IsValid());
	PackagesDialogWidget->SetWarning(InMessage);
}

/**
 * Populates the passed in array with the desired packages
 *
 * @param	OutPackages	The array that should be populated with the desired packages
 * @param	InChecked	The type of packages that we want to retrieve
 */
void FPackagesDialogModule::GetResults(OUT TArray<UPackage*>& OutPackages, ECheckBoxState InChecked)
{
	if(InChecked == ECheckBoxState::Checked)
	{
		for(int32 PackageIndex = 0; PackageIndex < CheckedPackages.Num(); ++PackageIndex)
		{
			OutPackages.Add(CheckedPackages[PackageIndex]);
		}
	}
	else if(InChecked == ECheckBoxState::Unchecked)
	{
		for(int32 PackageIndex = 0; PackageIndex < UncheckedPackages.Num(); ++PackageIndex)
		{
			OutPackages.Add(UncheckedPackages[PackageIndex]);
		}
	}
	else if(InChecked == ECheckBoxState::Undetermined)
	{
		for(int32 PackageIndex = 0; PackageIndex < UndeterminedPackages.Num(); ++PackageIndex)
		{
			OutPackages.Add(UndeterminedPackages[PackageIndex]);
		}
	}
}

/**
 * Removes all package items from the dialog
 */
void FPackagesDialogModule::RemoveAllPackageItems()
{
	PackagesDialogWidget.Get()->RemoveAll();
}

/**
 * Adds a new item to the checkbox that represents a package
 *
 * @param	InPackage		The package that will be represented as a checkbox
 * @param	InEntryName		The name to display
 * @param	InChecked		The state of the checkbox
 * @param	InDisabled		If the item should be disabled
 * @param	InIconName		The name of the icon to display
 * @param	InIconToolTip	The tooltip to display
 */
void FPackagesDialogModule::AddPackageItem(UPackage* InPackage, ECheckBoxState InChecked, 
	bool InDisabled/*=false*/, FString InIconName/*=TEXT("SavePackages.SCC_DlgNoIcon")*/, FString InIconToolTip/*=TEXT("")*/)
{
	FString AssetDisplayName;
	UE::Core::FVersePath VersePath;
	FString OwnerName;

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();

	// Lookup for the asset in the package
	if (UObject* FoundAsset = InPackage->FindAssetInPackage())
	{
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(FoundAsset->GetClass());
		if (AssetTypeActions.IsValid())
		{
			AssetDisplayName = AssetTypeActions.Pin()->GetObjectDisplayName(FoundAsset);
		}
		else
		{
			AssetDisplayName = FoundAsset->GetName();
		}

		if (AssetToolsModule.Get().ShowingContentVersePath())
		{
			VersePath = FoundAsset->GetVersePath();
		}

		OwnerName = FoundAsset->GetOutermostObject()->GetName();
	}

	// Last resort, display the package name
	if (AssetDisplayName.IsEmpty())
	{
		AssetDisplayName = FPackageName::GetShortName(InPackage->GetFName());
	}

	const FString& Extension = (InPackage->GetPackageFlags() & PKG_ContainsMap) ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
	FString FileName = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(InPackage->GetName(), Extension));

	// Ensure that the filename is formatted in a way that the current platform natively understands. 
	FPaths::MakePlatformFilename(FileName);
	
	PackagesDialogWidget.Get()->Add(MakeShareable(new FPackageItem(InPackage, MoveTemp(AssetDisplayName), MoveTemp(FileName), MoveTemp(VersePath), MoveTemp(OwnerName), InChecked, InDisabled, MoveTemp(InIconName), MoveTemp(InIconToolTip))));
}

/**
 * Adds a new button to the package dialog window
 *
 * @param	Type		The type of this button
 * @param	Name		The name to display
 * @param	ToolTip		The tooltip to display
 * @param	Disabled	If the button should be disabled
 */
void FPackagesDialogModule::AddButton(EDialogReturnType Type, const FText& Name, const FText& ToolTip/*=FText()*/,  TAttribute<bool> Disabled /*= false*/)
{
	EDialogButtonStyle Style = (Type == DRT_Save) || (Type == DRT_CheckOut) ? DBS_Primary : DBS_Normal;
	PackagesDialogWidget.Get()->AddButton(MakeShareable(new FPackageButton(this, Type, Style, Name, ToolTip, Disabled)));
}

/**
 * Adds a new button to the package dialog window
 *
 * @param	Type		The type of this button
 * @param	Style		The style of this button
 * @param	Name		The name to display
 * @param	ToolTip		The tooltip to display
 * @param	Disabled	If the button should be disabled
 */
void FPackagesDialogModule::AddButton(EDialogReturnType Type, EDialogButtonStyle Style, const FText& Name, const FText& ToolTip/*=FText()*/, TAttribute<bool> Disabled /*= false*/)
{
	PackagesDialogWidget.Get()->AddButton(MakeShareable(new FPackageButton(this, Type, Style, Name, ToolTip, Disabled)));
}

/**
 * Checks to see if the window is currently initialized
 *
 * @return	True if initialized, otherwise false
 */
bool FPackagesDialogModule::IsWindowInitialized() const
{
	return EditorPackagesDialogWindow.IsValid();
}

/**
 * Called when a module has been changed (unloaded, loaded, etc)
 */
void FPackagesDialogModule::OnModulesChanged( FName ModuleThatChanged, EModuleChangeReason ReasonForChange )
{

}
