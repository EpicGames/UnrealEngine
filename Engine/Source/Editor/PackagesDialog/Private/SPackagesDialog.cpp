// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPackagesDialog.h"

#include "AssetRegistry/AssetData.h"
#include "AssetToolsModule.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "SWarningOrErrorBox.h"
#include "SlotBase.h"
#include "Algo/AnyOf.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleDefaults.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "UObject/UObjectHash.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SPackagesDialog"

namespace SPackagesDialogDefs
{
	const FName ColumnID_CheckBoxLabel( "CheckBox" );
	const FName ColumnID_IconLabel( "Icon" );
	const FName ColumnID_AssetLabel( "Asset" );
	const FName ColumnID_OwnerLabel( "Owner" );
	const FName ColumnID_PackageLabel( "Package" );
	const FName ColumnID_TypeLabel( "Type" );
	const FName ColumnID_CheckedOutByLabel( "CheckedOutBy" );

	const float CheckBoxColumnWidth = 38.0f;
	const float IconColumnWidth = 22.0f;
}


UObject* FPackageItem::GetPackageObject() const
{
	UObject* FoundObject = nullptr;
	if ( !FileName.StartsWith(TEXT("/Temp/Untitled")) )
	{
		FoundObject = Package->FindAssetInPackage();
	}
	return FoundObject;
}

bool FPackageItem::HasMultipleAssets() const
{
	if ( !FileName.StartsWith(TEXT("/Temp/Untitled")) )
	{
		int32 NumAssets = 0;
		int32 NumDeleted = 0;
		ForEachObjectWithPackage(Package, [&NumAssets,&NumDeleted](UObject* Obj)
			{
				if (Obj->IsAsset() && !UE::AssetRegistry::FFiltering::ShouldSkipAsset(Obj))
				{
					++NumAssets;

					if (!IsValid(Obj))
					{
						++NumDeleted;
					}
				}
				return true;
			}, EGetObjectsFlags::None);

		return (NumAssets - NumDeleted) > 1;
	}

	return false;
}

bool FPackageItem::GetTypeNameAndColor(FText& OutName, FColor& OutColor) const
{
	if(HasMultipleAssets())
	{
		OutColor = FColor::White;
		OutName = LOCTEXT("MultipleAssets", "Multiple Assets");
		return true;
	}

	// Resolve the object belonging to the package and cache.
	if (!Object.IsValid())
	{
		Object = GetPackageObject();
	}

	if (Object.IsValid(/*bEvenIfPendingKill*/true, /*bThreadsafeTest*/false))
	{
		// Load the asset tools module to get access to the class color
		UObject* ObjectPtr = Object.Get(/*bEvenIfPendingKill*/true);

		FAssetData AssetData(ObjectPtr);
		OutName = FText::FromString(AssetData.AssetClassPath.ToString());

		if (!IsValidChecked(ObjectPtr) || UPackage::IsEmptyPackage(GetPackage()))
		{
			OutColor = FColor::Red;
		}
		else
		{
			const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
			const TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(ObjectPtr->GetClass()).Pin();
			if ( AssetTypeActions.IsValid() )
			{
				OutColor = AssetTypeActions->GetTypeColor();
			}
			else
			{
				OutColor = FColor::White;
			}
		}
		return true;
	}

	// if we do not find any package object, consider the package empty, return a red `Empty Package`
	OutName = LOCTEXT("NoAssets", "Empty Package");
	OutColor = FColor(					// Copied from ContentBrowserCLR.cpp
		127 + FColor::Red.R / 2,		// Desaturate the colors a bit (GB colors were too.. much)
		127 + FColor::Red.G / 2,
		127 + FColor::Red.B / 2,
		200);		// Opacity

	return true;
}

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SPackagesDialog::Construct(const FArguments& InArgs)
{
	bSortDirty = false;
	bReadOnly = InArgs._ReadOnly.Get();
	bAllowSourceControlConnection = InArgs._AllowSourceControlConnection.Get();
	Message = InArgs._Message;
	Warning = InArgs._Warning;
	OnSourceControlStateChanged = InArgs._OnSourceControlStateChanged;
	SortByColumn = SPackagesDialogDefs::ColumnID_AssetLabel;
	SortMode = EColumnSortMode::Ascending;

	ButtonsBox = SNew( SHorizontalBox );

	if(bAllowSourceControlConnection)
	{
		ButtonsBox->AddSlot()
		.AutoWidth()
		.Padding( 2 )
		[
			SNew(SButton) 
				.Text(LOCTEXT("ConnectToSourceControl", "Connect To Revision Control"))
				.ToolTipText(LOCTEXT("ConnectToSourceControl_Tooltip", "Connect to a revision control system for tracking changes to your content and levels."))
				.ContentPadding(FMargin(10, 3))
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Visibility(this, &SPackagesDialog::GetConnectToSourceControlVisibility)
				.OnClicked(this, &SPackagesDialog::OnConnectToSourceControlClicked)
		];
	}

	TSharedRef< SHeaderRow > HeaderRowWidget = SNew( SHeaderRow );

	if (!bReadOnly)
	{
		// clang-format off
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column( SPackagesDialogDefs::ColumnID_CheckBoxLabel )
			[
				SNew(SBox)
				.Padding(FMargin(6,3,6,3))
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda(
						[this]()
						{
							bool bAllChecked = true;
							bool bAllUnchecked = true;
							for (const TSharedPtr<FPackageItem>& Item : Items)
							{
								if (Item->GetState() != ECheckBoxState::Checked)
								{
									bAllChecked = false;
								}
								if (Item->GetState() != ECheckBoxState::Unchecked)
								{
									bAllUnchecked = false;
								}
							}

							if (bAllChecked)
							{
								return ECheckBoxState::Checked;
							}
							else if (bAllUnchecked)
							{
								return ECheckBoxState::Unchecked;
							}
							else
							{
								return ECheckBoxState::Undetermined;
							}
						}
					)
					.OnCheckStateChanged_Lambda(
						[this](ECheckBoxState InNewState)
						{
							for (const TSharedPtr<FPackageItem>& Item : Items)
							{
								if (InNewState == ECheckBoxState::Checked)
								{
									if (Item->IsDisabled())
									{
										Item->SetState(ECheckBoxState::Undetermined);
									}
									else
									{
										Item->SetState(ECheckBoxState::Checked);
									}
								}
								else
								{
									Item->SetState(ECheckBoxState::Unchecked);
								}
							}

							ItemListView->RequestListRefresh();
						}
					)
				]
			]
			.FixedWidth( SPackagesDialogDefs::CheckBoxColumnWidth )
		);
		// clang-format on
	}

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column( SPackagesDialogDefs::ColumnID_IconLabel )
		[
			SNew(SSpacer)
		]
		.SortMode( this, &SPackagesDialog::GetColumnSortMode, SPackagesDialogDefs::ColumnID_IconLabel )
		.OnSort( this, &SPackagesDialog::OnColumnSortModeChanged )
		.FixedWidth( SPackagesDialogDefs::IconColumnWidth )
	);

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column( SPackagesDialogDefs::ColumnID_AssetLabel )
		.DefaultLabel( LOCTEXT("AssetColumnLabel", "Asset" ) )
		.SortMode( this, &SPackagesDialog::GetColumnSortMode, SPackagesDialogDefs::ColumnID_AssetLabel )
		.OnSort( this, &SPackagesDialog::OnColumnSortModeChanged )
		.FillWidth( 5.0f )
	);
/*

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SPackagesDialogDefs::ColumnID_OwnerLabel)
		.DefaultLabel(LOCTEXT("OwnerColumnLabel", "Owner"))
		.SortMode(this, &SPackagesDialog::GetColumnSortMode, SPackagesDialogDefs::ColumnID_OwnerLabel)
		.OnSort(this, &SPackagesDialog::OnColumnSortModeChanged)
		.FillWidth(7.0f)
	);
*/

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column( SPackagesDialogDefs::ColumnID_PackageLabel )
		.DefaultLabel( LOCTEXT("FileColumnLabel", "File" ) )
		.SortMode( this, &SPackagesDialog::GetColumnSortMode, SPackagesDialogDefs::ColumnID_PackageLabel )
		.OnSort( this, &SPackagesDialog::OnColumnSortModeChanged )
		.FillWidth( 7.0f )
	);

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column( SPackagesDialogDefs::ColumnID_TypeLabel )
		.DefaultLabel( LOCTEXT("TypeColumnLabel", "Type" ) )
		.SortMode( this, &SPackagesDialog::GetColumnSortMode, SPackagesDialogDefs::ColumnID_TypeLabel )
		.OnSort( this, &SPackagesDialog::OnColumnSortModeChanged )
		.FillWidth( 2.0f )
		);

	if (bAllowSourceControlConnection)
	{
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SPackagesDialogDefs::ColumnID_CheckedOutByLabel)
			.DefaultLabel(LOCTEXT("CheckedOutByColumnLabel", "Checked Out By"))
			.SortMode(this, &SPackagesDialog::GetColumnSortMode, SPackagesDialogDefs::ColumnID_CheckedOutByLabel)
			.OnSort(this, &SPackagesDialog::OnColumnSortModeChanged)
			.FillWidth(4.0f)
			);
	}

	// clang-format off
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(FMargin(16))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f,0.0f,0.0f,8.0f)
			[
				SNew(STextBlock)
				.Text(this, &SPackagesDialog::GetMessage)
				.AutoWrapText(true)
			]

			+SVerticalBox::Slot()
			.FillHeight(0.8)
			[
				SAssignNew(ItemListView, SListView< TSharedPtr<FPackageItem> >)
					.ListItemsSource(&Items)
					.OnGenerateRow(this, &SPackagesDialog::MakePackageListItemWidget)
					.OnContextMenuOpening(this, &SPackagesDialog::MakePackageListContextMenu)
					.HeaderRow( HeaderRowWidget )
					.SelectionMode( ESelectionMode::Multi )
			]
			+ SVerticalBox::Slot()
			.Padding(0, 16.0f, 0, 0)
			.AutoHeight()
			[
				SNew(SWarningOrErrorBox)
				.Visibility(this, &SPackagesDialog::GetWarningVisibility)
				.Message(this, &SPackagesDialog::GetWarning)
			]
			+SVerticalBox::Slot() 
			.AutoHeight()
			.Padding(0.0f, 16.0f, 0.0f, 0.0f)
			.HAlign(HAlign_Right) 
			.VAlign(VAlign_Bottom)
			[
				ButtonsBox.ToSharedRef()
			]
		]
	];
	// clang-format on
}

/**
 * Removes all checkbox items from the dialog
 */
void SPackagesDialog::RemoveAll()
{
	Items.Reset();
}

/**
 * Adds a new checkbox item to the dialog
 *
 * @param	Item	The item to be added
 */
void SPackagesDialog::Add(TSharedPtr<FPackageItem> Item)
{
	FSimpleDelegate RefreshCallback = FSimpleDelegate::CreateSP(this, &SPackagesDialog::RefreshButtons);
	Item->SetRefreshCallback(RefreshCallback);
	Items.Add(Item);
	RequestSort();
}

/**
 * Adds a new button to the dialog
 *
 * @param	Button	The button to be added
 */
void SPackagesDialog::AddButton(TSharedPtr<FPackageButton> Button)
{
	Buttons.Add(Button);

	ButtonsBox->AddSlot()
	.AutoWidth()
	.Padding(5, 0)
	[
		SNew(SButton) 
			.ButtonStyle(&FAppStyle::Get(), (Button->GetStyle() == DBS_Primary) ? "PrimaryButton" : "Button")
			.TextStyle(&FAppStyle::Get(), (Button->GetStyle() == DBS_Primary) ? "PrimaryButtonText" : "ButtonText")
			.Text(Button->GetName())
			.ToolTipText(Button->GetToolTip())
			.IsEnabled(Button.Get(), &FPackageButton::IsEnabled)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.OnClicked(Button.Get(), &FPackageButton::OnButtonClicked)
	];
}

/**
 * Sets the message of the widget
 *
 * @param	InMessage	The string that the message should be set to
 */
void SPackagesDialog::SetMessage(const FText& InMessage)
{
	Message = InMessage;
}

/**
* Sets the warning message of the widget
*
* @param	InMessage	The string that the warning message should be set to
*/
void SPackagesDialog::SetWarning(const FText& InWarning)
{
	Warning = InWarning;
}

/**
 * Gets the return type of the dialog and it populates the package array results
 *
 * @param	OutCheckedPackages		Will be populated with the checked packages
 * @param	OutUncheckedPackages	Will be populated with the unchecked packages
 * @param	OutUndeterminedPackages	Will be populated with the undetermined packages
 *
 * @return	returns the button that was pressed to remove the dialog
 */
EDialogReturnType SPackagesDialog::GetReturnType(OUT TArray<UPackage*>& OutCheckedPackages, OUT TArray<UPackage*>& OutUncheckedPackages, OUT TArray<UPackage*>& OutUndeterminedPackages)
{
	/** Set the return type to which button was pressed */
	EDialogReturnType ReturnType = DRT_None;
	for( int32 ButtonIndex = 0; ButtonIndex < Buttons.Num(); ++ButtonIndex)
	{
		FPackageButton& Button = *Buttons[ButtonIndex];
		if(Button.IsClicked())
		{
			ReturnType = Button.GetType();
			break;
		}
	}

	/** Populate the results */
	if(ReturnType != DRT_Cancel && ReturnType != DRT_None)
	{
		for( int32 ItemIndex = 0; ItemIndex < Items.Num(); ++ItemIndex)
		{
			FPackageItem& item = *Items[ItemIndex];
			if(item.GetState() == ECheckBoxState::Checked)
			{
				OutCheckedPackages.Add(item.GetPackage());
			}
			else if(item.GetState() == ECheckBoxState::Unchecked)
			{
				OutUncheckedPackages.Add(item.GetPackage());
			}
			else
			{
				OutUndeterminedPackages.Add(item.GetPackage());
			}
		}
	}

	return ReturnType;
}

/**
 * Gets the widget which is to have keyboard focus on activating the dialog
 *
 * @return	returns the widget
 */
TSharedPtr< SWidget > SPackagesDialog::GetWidgetToFocusOnActivate() const
{
	// Find the first visible button.  That will be our widget to focus
	FChildren* ButtonBoxChildren = ButtonsBox->GetChildren();
	for( int ButtonIndex = 0; ButtonIndex < ButtonBoxChildren->Num(); ++ButtonIndex )
	{
		TSharedPtr<SWidget> ButtonWidget = ButtonBoxChildren->GetChildAt(ButtonIndex);
		if(ButtonWidget.IsValid() && ButtonWidget->GetVisibility() == EVisibility::Visible)
		{
			return ButtonWidget;
		}
	}

	return  TSharedPtr< SWidget >();
}

/**
 * Called when the checkbox items have changed state
 */
void SPackagesDialog::RefreshButtons()
{
	int32 CheckedItems = 0;
	int32 UncheckedItems = 0;
	int32 UndeterminedItems = 0;

	/** Count the number of checkboxes that we have for each state */
	for( int32 ItemIndex = 0; ItemIndex < Items.Num(); ++ItemIndex)
	{
		FPackageItem& item = *Items[ItemIndex];
		if(item.GetState() == ECheckBoxState::Checked)
		{
			CheckedItems++;
		}
		else if(item.GetState() == ECheckBoxState::Unchecked)
		{
			UncheckedItems++;
		}
		else
		{
			UndeterminedItems++;
		}
	}

	/** Change the button state based on our selection */
	for( int32 ButtonIndex = 0; ButtonIndex < Buttons.Num(); ++ButtonIndex)
	{
		FPackageButton& Button = *Buttons[ButtonIndex];
		if(Button.GetType() == DRT_MakeWritable)
		{
			Button.SetDisabled(UndeterminedItems == 0 && CheckedItems == 0);
		}
		else if(Button.GetType() == DRT_CheckOut)
		{
			Button.SetDisabled(CheckedItems == 0);
		}
		else if (Button.GetType() == DRT_Skip)
		{
			Button.SetDisabled(CheckedItems > 0);
		}
	}
}

/** 
 * Makes the widget for the checkbox items in the list view 
 */
TSharedRef<ITableRow> SPackagesDialog::MakePackageListItemWidget( TSharedPtr<FPackageItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew( SPackageItemsListRow, OwnerTable )
			.PackagesDialog( SharedThis( this ) )
			.Item( Item );
}


TSharedRef<SWidget> SPackagesDialog::GenerateWidgetForItemAndColumn( TSharedPtr<FPackageItem> Item, const FName ColumnID ) const
{
	check(Item.IsValid());

	const FMargin RowPadding(3, 3, 3, 3);

	TSharedPtr<SWidget> ItemContentWidget;

	if (ColumnID == SPackagesDialogDefs::ColumnID_CheckBoxLabel)
	{
		// clang-format off
		ItemContentWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(10, 3, 6, 3))
			[
				SNew(SCheckBox)
				.IsChecked(Item.Get(), &FPackageItem::OnGetDisplayCheckState)
				.OnCheckStateChanged_Lambda(
					[this, WeakItem = Item.ToWeakPtr()](ECheckBoxState InNewState)
					{
						if (TSharedPtr<FPackageItem> PinnedItem = WeakItem.Pin())
						{
							const bool bAllIfNone = false;
							const TArray<TSharedPtr<FPackageItem>> SelectedItems = this->GetSelectedItems(bAllIfNone);
							const bool bItemIsSelected = SelectedItems.Contains(PinnedItem);

							// If this item is selected, check/uncheck it together with all other checked items.
							if (bItemIsSelected)
							{
								for (const TSharedPtr<FPackageItem>& SelectedItem : SelectedItems)
								{
									SelectedItem->SetState(InNewState);
								}
							}
							// If this item is NOT in the current selection, just check/uncheck the current item.
							else
							{
								PinnedItem->SetState(InNewState);
							}
						}
					}
				)
			];
		// clang-format on
	}
	else if (ColumnID == SPackagesDialogDefs::ColumnID_IconLabel)
	{
		// Choose the icon based on the severity
		const FSlateBrush* IconBrush = Item->GetIconName().IsEmpty() ? FStyleDefaults::GetNoBrush() : FAppStyle::GetBrush(*(Item->GetIconName()));

		ItemContentWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image( IconBrush )
				.ToolTipText(FText::FromString(Item->GetToolTip()))
				.IsEnabled(!Item->IsDisabled())
			];
	}
	else if (ColumnID == SPackagesDialogDefs::ColumnID_AssetLabel)
	{
		const FString& AssetDisplayName = Item->GetAssetDisplayName();
		const FSlateIcon& AssetIcon = Item->GetAssetIcon();
		FText TypeName;
		FColor TypeColor;
		Item->GetTypeNameAndColor(TypeName, TypeColor);

		ItemContentWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(RowPadding)
			[
				SNew(SBox)
				.MaxAspectRatio(1.f)
				.HeightOverride(16.f)
				.WidthOverride(16.f)
				[
					SNew(SImage)
					.ToolTipText(TypeName)
					.ColorAndOpacity(TypeColor)
					.Image(AssetIcon.GetIcon())
					.IsEnabled(!Item->IsDisabled())
				]
			]
			+SHorizontalBox::Slot()
			.Padding(RowPadding)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(AssetDisplayName))
				.IsEnabled(!Item->IsDisabled())
			];
	}
	else if (ColumnID == SPackagesDialogDefs::ColumnID_OwnerLabel)
	{
		const FString& OwnerName = Item->GetOwnerName();

		ItemContentWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(RowPadding)
			[
				SNew(STextBlock)
				.Text(FText::FromString(OwnerName))
				.ToolTipText(FText::FromString(OwnerName))
				.IsEnabled(!Item->IsDisabled())
			];
	}
	else if (ColumnID == SPackagesDialogDefs::ColumnID_PackageLabel)
	{
		const FString& PackageName = Item->GetPackageName();
		const FString& FileName = Item->GetFileName();
		const UE::Core::FVersePath& VersePath = Item->GetVersePath();

		// Prefer the Verse path if valid.  It will be invalid if we are not showing Verse paths.
		ItemContentWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(RowPadding)
			[
				SNew(STextBlock)
				.Text(FText::FromString(VersePath.IsValid() ? VersePath.ToString() : PackageName))
				.ToolTipText(FText::FromString(FileName))
				.IsEnabled(!Item->IsDisabled())
			];
	}
	else if (ColumnID == SPackagesDialogDefs::ColumnID_TypeLabel)
	{
		const FText PackageType = Item->GetTypeName();

		ItemContentWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(RowPadding)
			[
				SNew(STextBlock)
				.Text(PackageType)
				.ToolTipText(PackageType)
				.IsEnabled(!Item->IsDisabled())
			];
	}
	else if (ColumnID == SPackagesDialogDefs::ColumnID_CheckedOutByLabel)
	{
		check(bAllowSourceControlConnection);

		FString CheckedOutByString = Item->GetCheckedOutByString();

		ItemContentWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(RowPadding)
			[
				SNew(STextBlock)
				.Text(FText::FromString(CheckedOutByString))
				.ToolTipText(FText::FromString(CheckedOutByString))
				.IsEnabled(!Item->IsDisabled())
			];
	}

	return ItemContentWidget.ToSharedRef();
}

TSharedPtr<SWidget> SPackagesDialog::MakePackageListContextMenu() const
{
	FMenuBuilder MenuBuilder( true, NULL );

	const TArray< TSharedPtr<FPackageItem> > SelectedItems = GetSelectedItems( false );
	if( SelectedItems.Num() > 0 )
	{	
		MenuBuilder.BeginSection("FilePackage", LOCTEXT("PackageHeading", "Asset"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("SCCDiffAgainstDepot", "Diff Against Depot"),
				LOCTEXT("SCCDiffAgainstDepotTooltip", "Look at differences between your version of the asset and that in revision control."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &SPackagesDialog::ExecuteSCCDiffAgainstDepot ),
					FCanExecuteAction::CreateSP( this, &SPackagesDialog::CanExecuteSCCDiffAgainstDepot )
				)
			);	
			MenuBuilder.AddMenuEntry(
				LOCTEXT("SCCCopyFilePathToClipboard", "Copy File Path"),
				LOCTEXT("SCCCopyFilePathToClipboardTooltip", "Copies the file path on disk to the clipboard."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda( [SelectedItems] ()
					{
						TArray<FString> Paths;
						Paths.Reserve(SelectedItems.Num());
						for (const TSharedPtr<FPackageItem>& PackageItem: SelectedItems)
						{
							if (!PackageItem->GetFileName().IsEmpty())
							{
								Paths.Add(PackageItem->GetFileName());
							}
						}
						FPlatformApplicationMisc::ClipboardCopy(*FString::Join(Paths, LINE_TERMINATOR));
					} ),
					FCanExecuteAction::CreateLambda( [SelectedItems] ()
					{
						return Algo::AnyOf(SelectedItems, [](const TSharedPtr<FPackageItem>& PackageItem)
						{
							return !PackageItem->GetFileName().IsEmpty();
						});
					} )
				)
			);	
			MenuBuilder.AddMenuEntry(
				LOCTEXT("SCCCopyPackagePathToClipboard", "Copy Package Path"),
				LOCTEXT("SCCCopyPackagePathToClipboardTooltip", "Copies the package path to the clipboard."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda( [SelectedItems] ()
					{
						TArray<FString> Paths;
						Paths.Reserve(SelectedItems.Num());
						for (const TSharedPtr<FPackageItem>& PackageItem: SelectedItems)
						{
							if (!PackageItem->GetFileName().IsEmpty())
							{
								Paths.Add(PackageItem->GetPackageName());
							}
						}
						FPlatformApplicationMisc::ClipboardCopy(*FString::Join(Paths, LINE_TERMINATOR));
					} )
				)
			);	

			const FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
			if (AssetToolsModule.Get().ShowingContentVersePath())
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SCCCopyVersePathToClipboard", "Copy Verse Path"),
					LOCTEXT("SCCCopyVersePathToClipboardTooltip", "Copies the Verse path to the clipboard."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([SelectedItems]()
						{
							TArray<FString> VersePaths;
							VersePaths.Reserve(SelectedItems.Num());
							for (const TSharedPtr<FPackageItem>& PackageItem: SelectedItems)
							{
								if (PackageItem->GetVersePath().IsValid())
								{
									VersePaths.Add(PackageItem->GetVersePath().ToString());
								}
							}
							FPlatformApplicationMisc::ClipboardCopy(*FString::Join(VersePaths, LINE_TERMINATOR));
						} ),
						FCanExecuteAction::CreateLambda([SelectedItems]()
						{
							return Algo::AnyOf(SelectedItems, [](const TSharedPtr<FPackageItem>& PackageItem)
							{
								return PackageItem->GetVersePath().IsValid();
							});
						})
					)
				);
			}
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

bool SPackagesDialog::CanExecuteSCCDiffAgainstDepot() const
{
	return ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable();
}

void SPackagesDialog::ExecuteSCCDiffAgainstDepot() const
{
	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

	const TArray< TSharedPtr<FPackageItem> > SelectedItems = GetSelectedItems( false );
	for(int32 ItemIdx=0; ItemIdx<SelectedItems.Num(); ItemIdx++)
	{
		const TSharedPtr<FPackageItem> SelectedItem = SelectedItems[ItemIdx];
		check( SelectedItem.IsValid() );

		UObject* Object = SelectedItem->GetPackageObject();
		if( Object )
		{
			const FString PackagePath = SelectedItem->GetFileName();
			const FString PackageName = FPaths::GetBaseFilename( PackagePath );
			AssetToolsModule.Get().DiffAgainstDepot( Object, PackagePath, PackageName );
		}
	}
}

TArray< TSharedPtr<FPackageItem> > SPackagesDialog::GetSelectedItems( bool bAllIfNone ) const
{
	//get the list of highlighted packages
	TArray< TSharedPtr<FPackageItem> > SelectedItems = ItemListView->GetSelectedItems();
	if ( SelectedItems.Num() == 0 && bAllIfNone )
	{
		//If no packages are explicitly highlighted, return all packages in the list.
		SelectedItems = Items;
	}

	return SelectedItems;
}

void SPackagesDialog::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if(bSortDirty)
	{
		bSortDirty = false;

		// Sort the list of root items
		SortTree();

		ItemListView->RequestListRefresh();
	}
}


FReply SPackagesDialog::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if( InKeyEvent.GetKey() == EKeys::Escape )
	{
		for( int32 ButtonIndex = 0; ButtonIndex < Buttons.Num(); ++ButtonIndex )
		{
			FPackageButton& Button = *Buttons[ ButtonIndex ];
			if( Button.GetType() == DRT_Cancel )
			{
				return Button.OnButtonClicked();
			}
		}
	}

	return SCompoundWidget::OnKeyDown( MyGeometry, InKeyEvent );
}

EVisibility SPackagesDialog::GetConnectToSourceControlVisibility() const
{
	if(bAllowSourceControlConnection)
	{
		if(!ISourceControlModule::Get().IsEnabled() || !ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FReply SPackagesDialog::OnConnectToSourceControlClicked() const
{
	ISourceControlModule::Get().ShowLoginDialog(FSourceControlLoginClosed(), ELoginWindowMode::Modal);
	OnSourceControlStateChanged.ExecuteIfBound();
	return FReply::Handled();
}

void SPackagesDialog::PopulateIgnoreForSaveItems( const TSet<FString>& InIgnorePackages )
{
	for( auto ItItem=Items.CreateIterator(); ItItem; ++ItItem )
	{
		const FString& ItemName = (*ItItem)->GetFileName();

		const ECheckBoxState CheckedStatus = (InIgnorePackages.Find(ItemName) != NULL) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;

		if (!(*ItItem)->IsDisabled())
		{
			(*ItItem)->SetState(CheckedStatus);
		}
	}
}

void SPackagesDialog::PopulateIgnoreForSaveArray( OUT TSet<FString>& InOutIgnorePackages ) const
{
	for( auto ItItem=Items.CreateConstIterator(); ItItem; ++ItItem )
	{
		if((*ItItem)->GetState()==ECheckBoxState::Unchecked)
		{
			InOutIgnorePackages.Add( (*ItItem)->GetFileName() );
		}
		else
		{
			InOutIgnorePackages.Remove( (*ItItem)->GetFileName() );
		}
	}
}

void SPackagesDialog::Reset()
{
	for( int32 ButtonIndex = 0; ButtonIndex < Buttons.Num(); ++ButtonIndex)
	{
		Buttons[ButtonIndex]->Reset();
	}
}

FText SPackagesDialog::GetMessage() const
{
	return Message;
}

FText SPackagesDialog::GetWarning() const
{
	return Warning;
}

EVisibility SPackagesDialog::GetWarningVisibility() const
{
	return (Warning.IsEmpty()) ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
}

EColumnSortMode::Type SPackagesDialog::GetColumnSortMode( const FName ColumnId ) const
{
	if ( SortByColumn != ColumnId )
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void SPackagesDialog::OnColumnSortModeChanged( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode )
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;

	RequestSort();
}

void SPackagesDialog::RequestSort()
{
	bSortDirty = true;
}

void SPackagesDialog::SortTree()
{
	int32 Sign;
	if (SortMode == EColumnSortMode::Ascending)
	{
		Sign = 1;
	}
	else if (SortMode == EColumnSortMode::Descending)
	{
		Sign = -1;
	}
	else
	{
		return;
	}

	constexpr int32 MaxComparers = 3;
	TArray<TFunction<int32(const TSharedPtr<FPackageItem>&, const TSharedPtr<FPackageItem>&)>, TInlineAllocator<MaxComparers>> CompareColumns;
	CompareColumns.Reserve(MaxComparers);

	auto CompareAssetColumn = [](const TSharedPtr<FPackageItem>& A, const TSharedPtr<FPackageItem>& B)
	{
		return A->GetAssetDisplayName().Compare(B->GetAssetDisplayName());
	};

	// Prefer the Verse path if valid.  It will be invalid if we are not showing Verse paths.
	auto ComparePackageColumn = [](const TSharedPtr<FPackageItem>& A, const TSharedPtr<FPackageItem>& B)
	{
		if (A->GetVersePath().IsValid() && B->GetVersePath().IsValid())
		{
			return A->GetVersePath().Compare(B->GetVersePath());
		}
		else if (A->GetVersePath().IsValid())
		{
			return A->GetVersePath().ToString().Compare(B->GetPackageName());
		}
		else if (B->GetVersePath().IsValid())
		{
			return A->GetPackageName().Compare(B->GetVersePath().ToString());
		}
		else
		{
			return A->GetPackageName().Compare(B->GetPackageName());
		}
	};

	if (SortByColumn == SPackagesDialogDefs::ColumnID_AssetLabel)
	{
		CompareColumns.Emplace(MoveTemp(CompareAssetColumn));

		// Use package path as tie breaker.
		CompareColumns.Emplace(MoveTemp(ComparePackageColumn));
	}
	else if (SortByColumn == SPackagesDialogDefs::ColumnID_PackageLabel)
	{
		CompareColumns.Emplace(MoveTemp(ComparePackageColumn));
	}
	else
	{
		if (SortByColumn == SPackagesDialogDefs::ColumnID_IconLabel)
		{
			CompareColumns.Emplace([](const TSharedPtr<FPackageItem>& A, const TSharedPtr<FPackageItem>& B)
			{
				return A->GetIconName().Compare(B->GetIconName());
			});
		}
		else if (SortByColumn == SPackagesDialogDefs::ColumnID_OwnerLabel)
		{
			CompareColumns.Emplace([](const TSharedPtr<FPackageItem>& A, const TSharedPtr<FPackageItem>& B)
			{
				return A->GetOwnerName().Compare(B->GetOwnerName());
			});
		}
		else if (SortByColumn == SPackagesDialogDefs::ColumnID_TypeLabel)
		{
			CompareColumns.Emplace([](const TSharedPtr<FPackageItem>& A, const TSharedPtr<FPackageItem>& B)
			{
				return A->GetTypeName().CompareTo(B->GetTypeName());
			});
		}
		else if (SortByColumn == SPackagesDialogDefs::ColumnID_CheckedOutByLabel)
		{
			CompareColumns.Emplace([](const TSharedPtr<FPackageItem>& A, const TSharedPtr<FPackageItem>& B)
			{
				return A->GetCheckedOutByString().Compare(B->GetCheckedOutByString());
			});
		}
		else
		{
			return;
		}

		// Use asset name and package path as tie breakers.
		CompareColumns.Emplace(MoveTemp(CompareAssetColumn));
		CompareColumns.Emplace(MoveTemp(ComparePackageColumn));
	}

	Items.Sort([&CompareColumns, Sign](const TSharedPtr<FPackageItem>& A, const TSharedPtr<FPackageItem>& B)
	{
		for (const TFunction<int32(const TSharedPtr<FPackageItem>&, const TSharedPtr<FPackageItem>&)>& CompareColumn : CompareColumns)
		{
			const int32 Result = CompareColumn(A, B);
			if (Result != 0)
			{
				return Sign * Result < 0;
			}
		}
		return false;
	});
}

void SPackageItemsListRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	PackagesDialogWeak = InArgs._PackagesDialog;
	Item = InArgs._Item;

	SMultiColumnTableRow< TSharedPtr<FPackageItem> >::Construct(
		FSuperRowType::FArguments()
		,InOwnerTableView );
}

TSharedRef<SWidget> SPackageItemsListRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	// Create the widget for this item
	auto PackagesDialogShared = PackagesDialogWeak.Pin();
	if (PackagesDialogShared.IsValid())
	{
		return PackagesDialogShared->GenerateWidgetForItemAndColumn( Item, ColumnName );
	}

	// Packages dialog no longer valid; return a valid, null widget.
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
