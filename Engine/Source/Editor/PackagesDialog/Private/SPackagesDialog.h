// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Containers/VersePath.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "PackagesDialog.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class SCheckBox;
class SHorizontalBox;
class STableViewBase;
class SWidget;
class UObject;
struct FGeometry;
struct FKeyEvent;

/**
 * Represents a button that will dynamically be added to the package dialog window
 */
class FPackageButton : public TSharedFromThis<FPackageButton>
{
public:
	FPackageButton(FPackagesDialogModule* InModule, EDialogReturnType InType, EDialogButtonStyle InStyle, const FText& InName, const FText& InToolTip, TAttribute<bool> InDisabled = false)
		: Module(InModule)
		, Name(InName)
		, ToolTip(InToolTip)
		, Type(InType)
		, Style(InStyle)
		, Clicked(false)
		, Disabled(InDisabled)
	{ }

	/**
	 * Gets called when the button is clicked
	 */
	FReply OnButtonClicked()
	{
 		Clicked = true;
		if (Module)
		{
			Module->RemovePackagesDialog();
		}
		return FReply::Handled();
	}

	/**
	 * Returns if the button should be enabled
	 *
	 * @return True if the button is not disabled, false otherwise
	 */
	bool IsEnabled()  const { return !Disabled.Get(); }

	/**
	 * Gets the name of the button
	 *
	 * @return the name of the button
	 */
	FText GetName() const { return Name; }

	/**
	 * Gets the tooltip for the button
	 *
	 * @return the tooltip for the button
	 */
	FText GetToolTip() const { return ToolTip; }

	/**
	 * Returns if the button was clicked
	 *
	 * @return True if the button was clicked, false otherwise
	 */
	bool IsClicked() const { return Clicked; }

	/**
	 * Gets the type of the button
	 *
	 * @return the type of the button
	 */
	EDialogReturnType GetType() const { return Type; }

	/**
	 * Gets the style of the button
	 *
	 * @return the style of the button
	 */
	EDialogButtonStyle GetStyle() const { return Style; }

	/**
	 * Sets if the button should be disabled
	 *
	 * @param	InDisabled	New disabled value
	 */
	void SetDisabled(bool InDisabled)
	{ 
		if( !Disabled.IsBound() )
		{
			Disabled.Set( InDisabled ); 
		}
	}

	/** 
	 * Resets this button state
	 */
	void Reset()
	{
		Clicked = false;
	}

private:
	FPackagesDialogModule* Module;	// Stores the module that contains this button
	FText Name;						// Name of the button
	FText ToolTip;					// Tool tip for this button
	EDialogReturnType Type;			// Button type
	EDialogButtonStyle Style;		// Button style
	bool Clicked;					// Stores if the button was clicked to close the dialog
	TAttribute<bool> Disabled;		// Stores if the button is disabled or not
};

/**
 * Represents package item that is displayed as a checkbox inside the package dialog
 */
class FPackageItem : public TSharedFromThis<FPackageItem>
{
public:
	FPackageItem(UPackage* InPackage, FString&& InAssetDisplayName, FString&& InFileName, UE::Core::FVersePath&& InVersePath, FString&& InOwnerName, ECheckBoxState InState, bool InDisabled, FString&& InIconName, FString&& InIconToolTip)
		: Package(InPackage)
		, AssetDisplayName(MoveTemp(InAssetDisplayName))
		, PackageName(FPackageName::ObjectPathToPackageName(InPackage->GetName()))
		, FileName(MoveTemp(InFileName))
		, VersePath(MoveTemp(InVersePath))
		, OwnerName(MoveTemp(InOwnerName))
		, State(InState)
		, Disabled(InDisabled)
		, IconName(MoveTemp(InIconName))
		, IconToolTip(MoveTemp(InIconToolTip))
	{
		/** if the item is checked and disabled make the state Undetermined */
		if(State == ECheckBoxState::Checked && Disabled)
		{
			State = ECheckBoxState::Undetermined;
		}
		
		// Find the appropriate icon
		if (const UObject* AssetObject = GetPackageObject())
		{
			AssetIcon = FSlateIconFinder::FindIconForClass(AssetObject->GetClass());
		}

		// Falling back on default icon
		if (!AssetIcon.IsSet())
		{
			AssetIcon = FSlateIconFinder::FindIconForClass(UObject::StaticClass());
		}
	}

	/**
	 * @return FSlateIcon the icon for the underlying asset object class or default object icon
	 */
	const FSlateIcon& GetAssetIcon() const { return AssetIcon; }

	/**
	 * Gets the display state of the item
	 *
	 * @return the item state
	 */
	ECheckBoxState OnGetDisplayCheckState() const
	{
		RefreshButtonCallback.ExecuteIfBound();
		return State;
	}

	/**
	 * Sets the item state
	 *
	 * @param InNewState the new state to set the item to
	 */
	void OnDisplayCheckStateChanged(ECheckBoxState InNewState)
	{
		State = InNewState;

		/** if the item is checked and disabled make the state Undetermined */
		if( State == ECheckBoxState::Checked  && Disabled )
		{
			State = ECheckBoxState::Undetermined;
		}
	}

	/**
	 * Sets refresh callback that should be called when the item's state change
	 *
	 * @param InRefreshButtonCallback the new callback
	 */
	void SetRefreshCallback(FSimpleDelegate InRefreshButtonCallback)
	{
		RefreshButtonCallback = InRefreshButtonCallback;
	}

	/**
	 * Gets the state of the checkbox item
	 *
	 * @return the state of the checkbox item
	 */
	ECheckBoxState GetState() const { return State; }

	/**
	 * Gets the package represented by this checkbox item
	 *
	 * @return the package represented by this checkbox item
	 */
	UPackage* GetPackage() const { return Package; }

	/** 
	 * Get the object belonging to the package, if any
	 *
	 * @return the object which is in the package, if any
	 */
	UObject* GetPackageObject() const;


	/** 
	 * Get whether the package contains multiple assets
	 *
	 * @return whether the package contains multiple assets
	 */
	bool HasMultipleAssets() const;

	/**
	 * Checks to see if the checkbox item is disabled
	 *
	 * @return True if the checkbox item is disabled, false otherwise
	 */
	bool IsDisabled() const { return Disabled; }

	/**
	 * Gets the display name of the primary asset item
	 *
	 * @return the display name of the asset item
	 */
	const FString& GetAssetDisplayName() const { return AssetDisplayName; }

	/**
	 * Gets the name of the package item
	 *
	 * @return the name of the package item
	 */
	const FString& GetPackageName() const { return PackageName; }

	/**
	 * Gets the name of the file item
	 *
	 * @return the name of the file item
	 */
	const FString& GetFileName() const { return FileName; }

	/**
	 * Gets the Verse path of the primary asset item
	 *
	 * @return the Verse path of the primary asset item
	 */
	const UE::Core::FVersePath& GetVersePath() const { return VersePath; }

	/**
	 * Gets the name of the file owner item
	 *
	 * @return the name of the file owner item
	 */
	const FString& GetOwnerName() const { return OwnerName; }

	/**
	 * Gets the icon name of the checkbox item
	 *
	 * @return the icon name of the checkbox item
	 */
	const FString& GetIconName() const { return IconName; }

	/**
	 * Get a string containing the name(s) of other users who have the file checked out
	 *
	 * @return names of other users who have the file checked out
	 */
	FString GetCheckedOutByString() const
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Package, EStateCacheUsage::Use);

		FString CheckedOutBy;
		if (SourceControlState.IsValid())
		{
			SourceControlState->IsCheckedOutOther(&CheckedOutBy);

			// If not checked out in this branch, see if checked out or modified in other branches
			if (!CheckedOutBy.Len())
			{
				// Check for modification first, as this is a more important state than only checked out
				if (SourceControlState->IsModifiedInOtherBranch())
				{
					FString HeadBranch, HeadAction;
					int32 HeadCL;
					SourceControlState->GetOtherBranchHeadModification(HeadBranch, HeadAction, HeadCL);

					FNumberFormattingOptions NoCommas;
					NoCommas.UseGrouping = false;
					return FString::Format(TEXT("Modified in {0} CL:{1} ({2})"), { HeadBranch, FText::AsNumber(HeadCL, &NoCommas).ToString(), HeadAction });
				}

				if (SourceControlState->IsCheckedOutInOtherBranch())
				{
					return SourceControlState->GetOtherUserBranchCheckedOuts();
				}
			}
		}

		return CheckedOutBy;
	}

	/**
	 * Gets the type name and color of the package item
	 *
	 * @param OutName	FText into which the type name will be placed, or an empty string if type cannot be obtained
	 * @param OutColor	FColor into which the type color will be placed
	 *
	 * @return Whether the details were successfully fetched.
	 */
	bool GetTypeNameAndColor(FText& OutName, FColor& OutColor) const;

	/**
	 * Gets just the type name of the package item
	 *
	 * @return Type name of the package item, or an empty string
	 */
	FText GetTypeName() const
	{
		FText OutName;
		FColor OutColor;
		GetTypeNameAndColor(OutName, OutColor);
		return OutName;
	}

	/**
	 * Gets the tool tip of the checkbox item
	 *
	 * @return the tool tip of the checkbox item
	 */
	FString GetToolTip() const { return IconToolTip; }

	/**
	 * Sets the new checkbox item state
	 *
	 * @param	NewState	New state
	 */
	void SetState(ECheckBoxState NewState) { State = NewState; }

private:
	UPackage* Package; 						// The package associated with this entry
	FString AssetDisplayName;				// Name of the asset to display
	FString PackageName;					// Name of the package to display
	FString FileName;						// Name of the file
	UE::Core::FVersePath VersePath;			// Verse path to display
	FString OwnerName;						// Name of the owner of the file
	ECheckBoxState State;					// The state of the checkbox
	bool Disabled; 							// if the entry is disabled
	FString IconName; 						// Name of an icon to show next to the checkbox
	FString IconToolTip;					// ToolTip to display for the icon
	FSimpleDelegate RefreshButtonCallback;	// ToolTip to display for the icon
	mutable TWeakObjectPtr<UObject> Object;	// Cached object associated with this entry.

	/** Icon for the asset class */
	FSlateIcon AssetIcon;
};

/**
 * Represents a package dialog comprised of packages and checkboxes and buttons
 */
class SPackagesDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPackagesDialog)
		: _ReadOnly(false)
		, _AllowSourceControlConnection(false)
		, _Message()
		, _Warning()
		{}

		/** When true, this dialog only shows a list of packages without the ability to filter */
		SLATE_ATTRIBUTE(bool, ReadOnly)

		/** When true, this dialog displays a 'connect to source control' button */
		SLATE_ATTRIBUTE(bool, AllowSourceControlConnection)

		/** The message of the widget */
		SLATE_ARGUMENT(FText, Message)

		/** The warning message of the widget */
		SLATE_ARGUMENT(FText, Warning)

		/** Called when source control state changes */
		SLATE_EVENT(FSimpleDelegate, OnSourceControlStateChanged)

	SLATE_END_ARGS()
	
	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs );

	/**
	 * Create and return a widget for the given item and column ID
	 *
	 * @param	Item		The item being queried
	 * @param	ColumnID	The column ID being queried
	 *
	 * @return	The widget which was created
	 */
	TSharedRef<SWidget> GenerateWidgetForItemAndColumn( TSharedPtr<FPackageItem> Item, const FName ColumnID ) const;

	/**
	 * Removes all checkbox items from the dialog
	 */
	void RemoveAll();

	/**
	 * Adds a new checkbox item to the dialog
	 *
	 * @param	Item	The item to be added
	 */
	void Add(TSharedPtr<FPackageItem> Item);

	/**
	 * Adds a new button to the dialog
	 *
	 * @param	Button	The button to be added
	 */
	void AddButton(TSharedPtr<FPackageButton> Button);

	/**
	 * Sets the message of the widget
	 *
	 * @param	InMessage	The string that the message should be set to
	 */
	void SetMessage(const FText& InMessage);

	/**
	 * Sets the warning message of the widget
	 *
	 * @param	InMessage	The string that the warning message should be set to
	 */
	void SetWarning(const FText& InMessage);

	/**
	 * Gets the return type of the dialog and it populates the package array results
	 *
	 * @param	OutCheckedPackages		Will be populated with the checked packages
	 * @param	OutUncheckedPackages	Will be populated with the unchecked packages
	 * @param	OutUndeterminedPackages	Will be populated with the undetermined packages
	 *
	 * @return	returns the button that was pressed to remove the dialog
	 */
	EDialogReturnType GetReturnType(OUT TArray<UPackage*>& OutCheckedPackages, OUT TArray<UPackage*>& OutUncheckedPackages, OUT TArray<UPackage*>& OutUndeterminedPackages);

	/**
	 * Gets the widget which is to have keyboard focus on activating the dialog
	 *
	 * @return	returns the widget
	 */
	TSharedPtr< SWidget > GetWidgetToFocusOnActivate() const;

	/**
	 * Get the visibility of the 'Connect to Source Control' button
	 */
	EVisibility GetConnectToSourceControlVisibility() const;

	/**
	 * Delegate used when the 'Connect to Source Control' button is clicked
	 */
	FReply OnConnectToSourceControlClicked() const;

	/**
	 * Populate the items with their current ignore status
	 * @param	InIgnorePackages	Container to populate the items from.
	 */
	void PopulateIgnoreForSaveItems( const TSet<FString>& InIgnorePackages );

	/**
	 * Populate current ignore status array with the item status
	 * @param	InOutIgnorePackages	Container to populate with the current ignore status of the items.
	 */
	void PopulateIgnoreForSaveArray( OUT TSet<FString>& InOutIgnorePackages ) const;

	/**
	 * Reset the state of this dialogs buttons
	 */
	void Reset();

	/**
	 * Whether the dialog allows a source control connection
	 */
	bool IsSourceControlConnectionAllowed() const { return bAllowSourceControlConnection; }

private:

	//~ Begin SWidget Interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	//~ End SWidget Interface

	/**
	 * Called when the checkbox items have changed state
	 */
	void RefreshButtons();

	/** 
	 * Makes the widget for the checkbox items in the list view 
	 */
	TSharedRef<ITableRow> MakePackageListItemWidget(TSharedPtr<FPackageItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** 
	 * Makes the widget for the context menu in the list view 
	 */
	TSharedPtr<SWidget> MakePackageListContextMenu() const;

	/** 
	 * Handler to check to see if "Diff Against Depot" can be executed 
	 */
	bool CanExecuteSCCDiffAgainstDepot() const;

	/** 
	 * Handler for when "Diff Against Depot" is selected
	 */
	void ExecuteSCCDiffAgainstDepot() const;

	/** 
	 * Get all the selected items in the dialog.
	 * 
	 * @param bAllIfNone - if true, returns all items when none are selected
	 * @return the array of selected packages
	 */
	TArray< TSharedPtr<FPackageItem> > GetSelectedItems( bool bAllIfNone ) const;

	/** Delegate used to supply message text to the widget */
	FText GetMessage() const;

	/** Delegate used to supply warning text to the widget */
	FText GetWarning() const;

	/** Delegate used to determine visibility of the warning */
	EVisibility GetWarningVisibility() const;

	/**
	 * Returns the current column sort mode (ascending or descending) if the ColumnId parameter matches the current
	 * column to be sorted by, otherwise returns EColumnSortMode_None.
	 *
	 * @param	ColumnId	Column ID to query sort mode for.
	 *
	 * @return	The sort mode for the column, or EColumnSortMode_None if it is not known.
	 */
	EColumnSortMode::Type GetColumnSortMode( const FName ColumnId ) const;

	/**
	 * Callback for SHeaderRow::Column::OnSort, called when the column to sort by is changed.
	 *
	 * @param	ColumnId	The new column to sort by
	 * @param	InSortMode	The sort mode (ascending or descending)
	 */
	void OnColumnSortModeChanged( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode );

	/**
	 * Requests that the source list data be sorted according to the current sort column and mode,
	 * and refreshes the list view.
	 */
	void RequestSort();

	/**
	 * Sorts the source list data according to the current sort column and mode.
	 */
	void SortTree();

	/** All checkbox items stored in this widget for the list view */
	TArray< TSharedPtr<FPackageItem> > Items;

	/** The list view for showing all checkboxes */
	TSharedPtr< SListView< TSharedPtr<FPackageItem> > >ItemListView;

	/** All buttons stored in this widget */
	TArray< TSharedPtr<FPackageButton> > Buttons;

	/** A horizontal box that will contain all of the buttons */
	TSharedPtr< SHorizontalBox > ButtonsBox;

	/** Refresh callback that should be called when a checkbox item state change */
	FSimpleDelegate RefreshButtonsCallback;
	
	/** A horizontal box that will represent the message of the widget */
	TSharedPtr< SHorizontalBox > MessageBox;

	/** When true, this dialog only shows a list of packages without the ability to filter */
	bool bReadOnly;

	/** When true, this dialog displays a 'connect to source control' button */
	bool bAllowSourceControlConnection;

	/** When true, the warning message is displayed in the widget */
	bool bShowWarning;

	/** When true, items will be sorted on the next tick */
	bool bSortDirty;

	/** The message to display */
	FText Message;

	/** The warning to display */
	FText Warning;

	/** Specify which column to sort with */
	FName SortByColumn;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode;

	/** Called when source control state changes */
	FSimpleDelegate OnSourceControlStateChanged;
};

/** Widget that represents a row in the PackagesDialog's list view.  Generates widgets for each column on demand. */
class SPackageItemsListRow
	: public SMultiColumnTableRow< TSharedPtr< FPackageItem > >
{

public:

	SLATE_BEGIN_ARGS( SPackageItemsListRow ) {}

		/** The Packages Dialog that owns the tree.  We'll only keep a weak reference to it. */
		SLATE_ARGUMENT( TSharedPtr< SPackagesDialog >, PackagesDialog )

		/** The list item for this row */
		SLATE_ARGUMENT( TSharedPtr< FPackageItem >, Item )

	SLATE_END_ARGS()


	/** Construct function for this widget */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView );

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;


private:

	/** Weak reference to the PackagesDialog widget that owns our list */
	TWeakPtr< SPackagesDialog > PackagesDialogWeak;

	/** The item associated with this row of data */
	TSharedPtr< FPackageItem > Item;
};
