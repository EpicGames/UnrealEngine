// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleInterface.h"

class SPackagesDialog;
class SWindow;
enum class ECheckBoxState : uint8;

enum EDialogReturnType
{
	DRT_Save			= 0,
	DRT_DontSave,
	DRT_CheckOut,
	DRT_MakeWritable,
	DRT_Cancel,
	DRT_Skip,
	DRT_Revert,
	DRT_None
};

enum EDialogButtonStyle
{
	DBS_Normal			= 0,
	DBS_Primary,
};

class FPackagesDialogModule : public IModuleInterface
{

public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

	/**
	 * Used to create the package dialog window
	 *
	 * @param	Title							The title of the package dialog window
	 * @param	Message							The message that gets displayed in the package dialog window
	 * @param	InReadOnly						When true, this dialog only shows a list of packages without the ability to filter
	 * @param	InAllowSourceControlConnection	When true, this dialog displays a 'connect to source control' button when needed
	 * @param	InOnSourceControlStateChanged	Delegate called when the source control state changes
	 */
	virtual void CreatePackagesDialog(const FText& Title, const FText& Message, bool InReadOnly = false, bool InAllowSourceControlConnection = false, const FSimpleDelegate& InOnSourceControlStateChanged = FSimpleDelegate());

	/**
	 * Shows the package dialog window as a modal window
	 *
	 * @return	Which button was pressed to terminate the window
	 */
	virtual EDialogReturnType ShowPackagesDialog();

	/**
	 * Shows the package dialog window as a modal window
	 *
	 * @param	InPackagedToIgnore	The Set of packages to ignore when saving.
	 *
	 * @return	Which button was pressed to terminate the window
	 */
	virtual EDialogReturnType ShowPackagesDialog(OUT TSet<FString>& InOutIgnoredPackages);

	/**
	 * Removes the package dialog window
	 */
	virtual void RemovePackagesDialog();

	/**
	 * Sets the message displayed in the package dialog
	 * @param InMessage		The message to display
	 */
	virtual void SetMessage(const FText& InMessage);

	/**
	 * Sets the warning message displayed in the package dialog
	 * @param InMessage		The warning to display
	 */
	virtual void SetWarning(const FText& InMessage);

	/**
	 * Populates the passed in array with the desired packages
	 *
	 * @param	OutPackages		The array that should be populated with the desired packages
	 * @param	InChecked		The type of packages that we want to retrieve
	 */
	virtual void GetResults(TArray<UPackage*>& OutPackages, ECheckBoxState InChecked);

	/**
	 * Removes all package items from the dialog
	 */
	virtual void RemoveAllPackageItems();

	/**
	 * Adds a new item to the checkbox that represents a package
	 *
	 * @param	InPackage		The package that will be represented as a checkbox
	 * @param	InChecked		The state of the checkbox
	 * @param	InDisabled		If the item should be disabled
	 * @param	InIconName		The name of the icon to display
	 * @param	InIconToolTip	The tooltip to display
	 */
	virtual void AddPackageItem(UPackage* InPackage, ECheckBoxState InChecked, bool InDisabled = false, FString InIconName=TEXT(""), FString InIconToolTip=TEXT(""));

	/**
	 * Adds a new button to the package dialog window
	 *
	 * @param	Type		The type of this button
	 * @param	ButtonStyle	The style of this button
	 * @param	Name		The name to display
	 * @param	ToolTip		The tooltip to display
	 * @param	Disabled	If the button should be disabled
	 */
	virtual void AddButton(EDialogReturnType Type, const FText& Name, const FText& ToolTip = FText(), TAttribute<bool> Disabled = false);
	virtual void AddButton(EDialogReturnType Type, EDialogButtonStyle ButtonStyle, const FText& Name, const FText& ToolTip = FText(), TAttribute<bool> Disabled = false);

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return	Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	/**
	 * Checks to see if the package dialog window is currently initialized
	 *
	 * @return	True if initialized, otherwise false
	 */
	virtual bool IsWindowInitialized() const;
	
protected:
	/**
	 * Called when a module has been changed (unloaded, loaded, etc)
	 */
	void OnModulesChanged( FName ModuleThatChanged, EModuleChangeReason ReasonForChange );

private:
	/** A default window size for the package dialog */
	static const FVector2D DEFAULT_WINDOW_SIZE;

	/** Extra window width if source control connection is allowed */
	static const FVector2D EXTRA_WINDOW_WIDTH;

	/** Editor package dialog window */
	TWeakPtr<SWindow> EditorPackagesDialogWindow;

	/** Packages Dialog Widget to use in the package dialog */
	TSharedPtr<class SPackagesDialog> PackagesDialogWidget;

	/** The title of the dialog window */
	TAttribute<FText> PackageDialogTitle;

	/** The results of the dialog will be stored in these arrays */
	TArray<UPackage*> CheckedPackages;
	TArray<UPackage*> UncheckedPackages;
	TArray<UPackage*> UndeterminedPackages;
};
